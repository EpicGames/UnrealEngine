"use strict"
const fs = require('fs');
const path = require('path');
const crypto = require('crypto');

let makeDir = function(dir)
{
	if (fs.existsSync(dir))
		return false;
	fs.mkdirSync(dir);
	return true;
};

let getChecksum = function(file, type, callback)
{
	let hash = crypto.createHash(type);
	let stream = fs.createReadStream(file);
	stream.on('data', function(data) {
		hash.update(data);
	});
	stream.on('end', function() {
		let hashStr = hash.digest('hex');
		callback(null, hashStr);
	});
};

let copyBuildFiles = function(BuildBaseDir, CdnBaseDir, OnComplete)
{
	// figure out the build id
	const BuilderBuildId = path.basename(BuildBaseDir);
	let m = BuilderBuildId.match(/\+\+([^\+]+)\+(.+)-CL-(\d+)/);
	if (m == null)
		throw new Error(`Build ID '${BuilderBuildId}' doesn't match expected format.`);

	// compute build ID
	let BuildId;
	let rm = m[2].match(/Release-(\d+)\.(\d+)/);
	if (rm != null)
		BuildId = `v${rm[1]}.${rm[2]}-r${m[3]}`;
	else
		BuildId = `${m[1].toLowerCase()}.${m[2].toLowerCase()}-r${m[3]}`;

	// build base dirs
	const CdnOutputDir = path.resolve(CdnBaseDir, BuildId);
	const BuildPaksDir = path.resolve(BuildBaseDir, "PakFiles");

	// log
	console.log(`BuildId = ${BuildId}`);
	console.log(`BuildPaksDir = ${BuildPaksDir}`);
	console.log(`CdnOutputDir = ${CdnOutputDir}`);

	// make sure the output dir exists
	makeDir(CdnOutputDir);

	// keep track of pending copies
	let pendingCopies = 1;
	let onCopyFinished = function() {
		--pendingCopies;
		if (pendingCopies <= 0)
		{
			if (pendingCopies < 0)
				throw new Error("Too many callbacks");
			if (OnComplete)
				OnComplete(CdnOutputDir);
		}
	};

	// find all the platforms
	let didWindows = false;
	for (let platform of fs.readdirSync(BuildPaksDir))
	{
		let platformDir = path.resolve(BuildPaksDir, platform);
		if (!fs.statSync(platformDir).isDirectory())
			continue;

		// make the output dir
		let outPlatform = platform;
		if (outPlatform.startsWith("Windows"))
		{
			if (didWindows)
			{
				console.log("Skipping ", platform);
				continue;
			}
			didWindows = true;
			outPlatform = "Windows"; // remove "NoEditor", "Client", and "Server"
		}
		let outDir = path.resolve(CdnOutputDir, outPlatform);
		makeDir(outDir);

		for (let chunkFile of fs.readdirSync(platformDir))
		{
			// parse out the chunk id
			let m = chunkFile.match(/pakchunk([0-9]+)/);
			if (m === null)
				continue;

			let chunkId = parseInt(m[1]);
			let chunkDir = path.resolve(platformDir, chunkFile);

			// ordered list of files in this dir
			let pakFileList = fs.readdirSync(chunkDir);
			pakFileList.sort();
			let pakNum = 0;
			for (let fileName of pakFileList)
			{
				if (!fileName.endsWith('.pak'))
					continue;

				// move the file to the cdn staging folder
				++pakNum;
				let pakFileName = `chunk${chunkId}-pak${pakNum}.pak`;
				let destFile = path.resolve(outDir, pakFileName);
				if (fs.existsSync(destFile))
				{
					console.log(`Skipping ${fileName} (already present)`)
					continue;
				}

				// do the copy
				let pakFile = path.resolve(chunkDir, fileName);
				++pendingCopies;
				fs.copyFile(pakFile, destFile, (err) => {
					if (err) throw err;
					console.log(`${fileName} copied to ${destFile}`);
					onCopyFinished();
				});
			}
		}
	}

	// counteract the 1 we added (helps us fire onComplete when there are 0 queued)
	onCopyFinished();
}

let generateManifests = function(CdnStageDir)
{
	// figure out the buildId from CdnStageDir
	let buildId = path.basename(CdnStageDir);

	// find all the platforms
	for (let platform of fs.readdirSync(CdnStageDir))
	{
		let platformDir = path.resolve(CdnStageDir, platform);
		if (!fs.statSync(platformDir).isDirectory())
			continue;

		// open a BuildManifest for writing 
		let files = [];
		let filesComplete = 0;
		let tryWriteManifest = function() {
			++filesComplete;
			if (filesComplete >= files.length)
			{
				// sort by chunk id
				files.sort(function(a, b) {
					if (a.chunk === b.chunk)
						return a.name.localeCompare(b.name);
					return a.chunk < b.chunk ? -1 : 1;
				});

				// sync write to the local manifest
				{
					let fileName = path.resolve(platformDir, "EmbeddedManifest.txt");
					let manifest = fs.openSync(fileName, "w");
					fs.writeSync(manifest, `$NUM_ENTRIES = ${files.length}\n`);
					for (let file of files)
					{
						fs.writeSync(manifest, `${file.name}\t${file.size}\t${file.version}\t-1\t-\n`);
					}
					fs.closeSync(manifest);
					console.log("wrote", fileName);
				}

				// sync write to the build manifest
				{
					let fileName = path.resolve(CdnStageDir, `BuildManifest-${platform}.txt`);
					let manifest = fs.openSync(fileName, "w");
					fs.writeSync(manifest, `$BUILD_ID = ${buildId}\n`);
					fs.writeSync(manifest, `$NUM_ENTRIES = ${files.length}\n`);
					for (let file of files)
					{
						fs.writeSync(manifest, `${file.name}\t${file.size}\t${file.version}\t${file.chunk}\t${file.url}\n`);
					}
					fs.closeSync(manifest);
					console.log("wrote", fileName);
				}
			}
		};

		// enumerate files
		for (let pakFile of fs.readdirSync(platformDir))
		{
			// parse out the chunk id
			let m = pakFile.match(/chunk([0-9]+)-pak[0-9]+\.pak/);
			if (m === null)
				continue;
			let chunkId = parseInt(m[1]);

			// compute these
			let relativeUrl = `${platform}/${pakFile}`;
			let filePath = path.resolve(platformDir, pakFile);
			let fileSize = fs.statSync(filePath).size;

			// add to the files array
			let file = {
				name: pakFile,
				chunk: chunkId,
				size: fileSize,
				version: "",
				url: relativeUrl,
				filePath: filePath,
			};
			files.push(file);
		}

		// checksum all the files
		console.log(`Checksumming ${files.length} files for ${platform}...`);
		for (let file of files)
		{
			// async compute the checksum
			getChecksum(file.filePath, 'sha1', (err, checksum) => {
				file.version = "SHA1:" + checksum;
				tryWriteManifest();
			});
		}
	}
}

let operation = process.argv[2] || "help";
if (operation === "process")
{
	if (!process.argv[3])
		throw new Error('Missing BuildBaseDir argument');
	if (!process.argv[4])
		throw new Error('Missing CdnBaseDir argument');

	const BuildBaseDir = path.resolve(process.argv[3]);
	const CdnBaseDir = path.resolve(process.argv[4]);

	// copy the build
	copyBuildFiles(BuildBaseDir, CdnBaseDir, (CdnStageDir) => {
		// then generate manifests
		generateManifests(CdnStageDir);
	});
	
}
else if (operation === "move")
{
	if (!process.argv[3])
		throw new Error('Missing BuildBaseDir argument');
	if (!process.argv[4])
		throw new Error('Missing CdnBaseDir argument');

	// just copy the build
	const BuildBaseDir = path.resolve(process.argv[3]);
	const CdnBaseDir = path.resolve(process.argv[4]);
	copyBuildFiles(BuildBaseDir, CdnBaseDir);
}
else if (operation === "manifest")
{
	if (!process.argv[3])
		throw new Error('Missing CdnStageDir argument');

	// just generate manifests
	const CdnStageDir = path.resolve(process.argv[3]);
	generateManifests(CdnStageDir);
}
else
{
	// help or invalid params
	console.log("process <build_source> <cdn_stage> // does both a move and manifest generation");
	console.log("move <build_source> <cdn_stage> // copy pak files from a build, rename then, and organize for CDN");
	console.log("manifest <cdn_stage>/<build> // generate manifests for a CDN prep folder");
}
