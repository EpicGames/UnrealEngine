# Purpose of this Documentation

This documentation is meant to support and enhance the [official Gameplay Ability System Unreal Developer Community documentation](https://dev.epicgames.com/documentation/en-us/unreal-engine/gameplay-ability-system-for-unreal-engine).  In particular, this document lives in the code repository under [Gameplay Ability System plug-in folder](./) and thus any user reading this documentation can submit a pull request to clarify functionality, update inaccurate information, or work with the community to flesh out areas that are missing.

It is worth noting there are extensive resources that the wider end-user developer community has written.  One such source of knowledge is the [tranek GAS documentation](https://github.com/tranek/GASDocumentation) which is highly detailed and an excellent resource for implementation details, but risks falling out of date with new feature additions or changes.

# Overview of the Gameplay Ability System

The Gameplay Ability System is a framework for building abilities and interactions that Actors can own and trigger. This system is designed mainly for RPGs, action-adventure games, MOBAs, and other types of games where characters have abilities that need to coordinate mechanics, visual effects, animations, sounds, and data-driven elements, although it can be adapted to a wide variety of projects. The Gameplay Ability System also supports replication for multiplayer games, and can save developers a lot of time scaling up their designs to support multiplayer.

The concepts that the Gameplay Ability System uses are:

- [Gameplay Attributes](#gameplay-attributes):  An enhancement to float properties that allow them to be temporarily modified (buffed) and used in complex calculations such as damage.
- [Gameplay Tags](#gameplay-tags):  A hierarchical naming system that allows you to specify states of Actors, and properties of Assets.  A powerful query system allows designers to craft logic statements around these.
- [Gameplay Cues](#gameplay-cues):  A visual and audio effects system based on Gameplay Tags which allow decoupling of the FX and the implementation.
- [Gameplay Abilities](#gameplay-abilities):  The code that actually triggers when an action is performed.  Typically a Blueprint graph.
- [Gameplay Effects](#gameplay-effects):  Predefined rulesets about how to apply all of the above.

One of the designers' often mentioned goals of the Gameplay Ability System is to maintain a record of who triggered a complex set of interactions, so that we can keep proper account of which Actor did what.  For instance, if a Player activates a [Gameplay Ability](#gameplay-abilities) that spawns a poison cloud (possibly represented with a [Gameplay Cue](#gameplay-cue)) that then does damage-over-time using a [Gameplay Effect](#gameplay-effects) which eventually reduces an Actor's Health [Gameplay Attribute](#gameplay-attributes) to zero, we should be able to award the kill to the initiating Player.

It is worth mentioning the damage system upfront, as it's a pervasive example throughout the documentation.  You may be familiar with the [AActor::TakeDamage function](https://www.unrealengine.com/blog/damage-in-ue4) which was used for many years.  At Epic, we no longer use that system internally; instead all damage is done through the Gameplay Ability System.  By using the Gameplay Ability System, we allow buffs/debuffs and an extensive and ever-changing list of damage types based on Gameplay Tags.  You can look at [Lyra](https://dev.epicgames.com/documentation/unreal-engine/lyra-sample-game-in-unreal-engine) as an example that uses the Gameplay Ability System extensively, as well as a rudimentary damage system.

---

# Ability System Component {#asc}

The Ability System Component (commonly abbreviated ASC) is the top-level ActorComponent that you use to interface with the Gameplay Ability System (commonly abbreviated GAS).  It is a monolithic class that encapsulates almost all of the functionality GAS uses.  By funneling all of the functionality through the ASC, we are able to better encapsulate and enforce the rules about activation, replication, prediction, and side-effects.

While the Ability System Component *is* an ActorComponent, we typically recommend against putting it on a Player's Pawn.  Instead, for a Player, it should be on the PlayerState.  The reason for this is that the Pawn is typically destroyed upon death in multiplayer games, and GAS typically has functionality (be it [Gameplay Attributes](#gameplay-attributes), [Gameplay Tags](#gameplay-tags), or [Gameplay Abilities](#gameplay-abilities)) that should persist beyond death.  For non-player AI-driven characters (e.g. AI that are not bots), it is suitable to put the ASC on the Pawn because it needs to replicate data to [Simulated Proxies](https://dev.epicgames.com/documentation/en-us/unreal-engine/actor-role-and-remote-role-in-unreal-engine#actorrolestates).

---

# Gameplay Attributes {#gameplay-attributes}

Gameplay Attributes (often just referred to as simply *Attributes*) are essentially *float* properties that are wrapped in a FGameplayAttributeData instance.  The reason for doing so is to allow for a *BaseValue* which one can think of as an unaltered intrinsic value of the Actor, and a *CurrentValue* which one can think of as the value that currently applies, after all of the buffs and debuffs of the Actor are taken into account.  These *Attributes* must live in an [AttributeSet](#attribute-sets).  There is Editor tooling around the use of *Attributes* that allow them to be selected and used inside [Gameplay Effects](#gameplay-effects) (and others) to ensure buffs and debuffs work correctly.

Attributes are often replicated, thus keeping the client in sync with the server values, but that does not *always* need to be the case.  For instance, certain *meta-Attributes* can be used to store temporary data used for calculations, allowing these intermediate results to have full buff/debuff aggregation capabilities; these *meta-Attributes* are not replicated because they are typically reset after a calculation.

Since Gameplay Attributes are easily accessible through native or Blueprint code, it's tempting to modify them directly.  However, the Gameplay Ability System is designed such that all modifications to the Attributes should be done through [Gameplay Effects](#gameplay-effects) to ensure they can be network predicted and rolled-back gracefully.

[Developer Community Gameplay Attribute & AttributeSet Docs](https://dev.epicgames.com/documentation/unreal-engine/gameplay-attributes-and-attribute-sets-for-the-gameplay-ability-system-in-unreal-engine)

## AttributeSets {#attribute-sets}

AttributeSets are simply classes derived from [UAttributeSet class](./Source/GameplayAbilities/Public/AttributeSet.h).  The AttributeSets typically contain multiple Gameplay Attributes that encompass all properties for a specific game feature (such as a jetpack item, but the most commonly cited example is the damage system).  AttributeSets must be added to the [Ability System Component](#asc) by the server.  AttributeSets are typically replicated to the client, but not all Attributes are replicated to the client (they are configured on a per-Attribute basis).

## Attribute Modifiers {#attribute-modifiers}

Attribute Modifiers are how we buff and debuff Attributes.  These are setup through the [Gameplay Effects'](#gameplay-effects) `Modifiers` property.  Once a modifier is 'active', it is stored in the [Ability System Component](#asc) and all requests for the value go through a process called *aggregation*.

The rules for *aggregation* can be unexpected to a new user.  For instance, if there are multiple values that modify a single attribute, the modifiers are added together before the result is computed.  Let's take an example of a multiplier of 10% added to damage, and another multiplier of 30% added to damage.  If one were purely looking at the numbers, one could think `Damage * 1.1 * 1.3 = 1.43` thus damage would be modified by *43%*.  However, the system takes these modifier operators into account and adds them separately before performing the final multiplier calculation, giving an expected result of `10% + 30% = 40%`.

---

# Gameplay Abilities {#gameplay-abilities}

Gameplay Abilities are derived from the [UGameplayAbility class](./Source/GameplayAbilities/Public/GameplayAbility.h).  They define what an in-game ability does, what (if anything) it costs to use, when or under what conditions it can be used, and so on.  Because Gameplay Abilities are implemented in native or Blueprints, it can do anything a Blueprint graph can do.  Unlike traditional Blueprints, they are capable of existing as instanced objects running asynchronously -- so you can run specialized, multi-stage tasks (called [Gameplay Ability Tasks](#gameplay-ability-tasks).  Examples of Gameplay Abilities would be a character dash, or an attack.

Think of a Gameplay Ability as the bundle of functions that correspond to the action you're performing.  There are complex rules about who can activate them, how they activate, and how they are predicted (locally executed ahead of the server acknowledgement).  You trigger them through the Ability System Component (typically through a TryActivate function).  But they can also be triggered through complex interactions (if desired) such as through Gameplay Events, [Gameplay Tags](#gameplay-tags), [Gameplay Effects](#gameplay-effects), and Input (which the [ASC](#asc) handles internally).

[Developer Community Gameplay Abilities docs](https://dev.epicgames.com/documentation/unreal-engine/using-gameplay-abilities-in-unreal-engine)

## Gameplay Ability Tasks {#gameplay-ability-tasks}

Gameplay Abilities often make use of [Gameplay Ability Tasks](https://dev.epicgames.com/documentation/unreal-engine/gameplay-ability-tasks-in-unreal-engine).  Gameplay Ability Tasks are latent Blueprint nodes that allow your Gameplay Ability to 'pause' for the frame while it awaits some event.  They can also perform network functionality which hide complex implementation details from the Blueprint designer.

## Gameplay Ability Specs

The Gameplay Ability Specs are runtime-defined data which augment and tie together parameters used for the Gameplay Ability.  It serves two purposes:

1. To configure the Gameplay Ability parameters prior to giving/granting the ability.  For instance, it defines what 'Level' of the ability you are granting.
2. To store information about the granted Gameplay Ability that is shared between all instances of the Gameplay Ability.

A lot of the [Ability System Component](#asc)'s interface deals with Gameplay Ability Specs, or after being granted, typically a Gameplay Ability Spec Handle.  The Handles are a way to succinctly refer to a Gameplay Ability Spec in both native and Blueprint code without worrying about the dangers of holding onto a pointer (and having that pointer be reallocated).  Whenever you want to refer to an already-granted Ability, use an Ability Spec Handle instead.

The corresponding class for [FGameplayAbilitySpec](./Source/GameplayAbilities/Public/GameplayAbilitySpec.h) and [FGameplayAbilitySpecHandle](./Source/GameplayAbilities/Public/GameplayAbilitySpecHandle.h).

## Gameplay Ability Instancing Policy

The instancing policy determines when a Gameplay Ability is instanced (the Gameplay Ability object is created) and thus controls the lifetime of the GA.  The safest, and most feature-supported choice is InstancedPerActor.

### InstancedPerActor

When choosing InstancedPerActor, the Gameplay Ability will be instanced when its corresponding Gameplay Ability Spec is first given (granted) to the Actor.  The instance lives until the Gameplay Ability Spec is removed from the Actor.  This lifetime mimics what most users expect:  You are granted an ability and immediately have an instance of it.

The lifetime semantics come with some pitfalls you should be aware of:

- Since the ability continues to exist after it has ended, none of the variables will be reset for next activation.  Thus it's the user's responsibility to reset the variables to the defaults in EndAbility.
- Prior to UE5.5, you could receive function calls such as OnGiveAbility/OnRemoveAbility on the _instance_ immediately, before the ability had ever been activated.  This isn't true of the other instancing types, which execute said functions on the CDO.  UE5.5 deprecates such functions in favor of explicit execution on the CDO.
- There is a function you may see often called GetPrimaryInstance.  The Primary Instance refers to the InstancedPerActor's one-and-only instance; it does not apply to other instancing types.

### InstancedPerExecution

When choosing InstancedPerExecution, you receive a new instance of the Gameplay Ability for each and every activation.  Some things you should be aware of:

- The instancing happens on activation (not prior to it).  It is possible to Grant & Revoke an InstancedPerExecution ability without ever instancing it.
- Replicated Gameplay Abilities (GA's which contain RPC's or Replicated Variables) are relatively expensive, as a new GA must be sent for every activation.
- Unlike InstancedPerActor, an individual instance is always active (otherwise it would have not been created).  It is garbage collected immediately upon ending.

### NonInstanced (Deprecated)

Prior to UE5.5, we had functionality for Non-Instanced Gameplay Abilities.  Since these Gameplay Abilities were never instanced, they could not be replicated or even hold state (e.g. contain variables).  All functions were called on the ClassDefaultObject and thus all state had to be held on the Gameplay Ability Spec.  This made them very confusing to use.  The same functionality can be achieved by simply using InstancedPerActor and never revoking it; the cost is just a single allocation (instance) of a UGameplayAbility.

## Replication

There is a replication policy variable on the Gameplay Abilities.  The setting controls whether or not you are able to use Remote Procedure Calls (RPC's) or Replicated Variables (now deprecated, see below).  It does *not* control if a Gameplay Ability will activate both on Server & Client -- that is controlled via the Execution Policy.

Keep in mind that Gameplay Abilities exist only on the locally controlled actors and on the server.  As such, you cannot replicate data meant to be visible on Simulated Proxies using Gameplay Abilities.  You would have to use other mechanisms, such as Attributes for replicated variables, or use RPC's directly on the Actors.

### Replicated Variables in Gameplay Abilities

The usage of replicated variables is deprecated as of UE5.5.  The deprecation warning is controlled by a Console Variable "AbilitySystem.DeprecateReplicatedProperties", so that users can turn off the warning and continue using the feature until they are ready to fix the issue.

The reasoning is to prevent users from stumbling upon an impossible-to-solve bug regarding replication ordering:

- Replicated variables are guaranteed to be delivered, but not in any particular order with respect to each other or RPC functions.
- Gameplay Ability activation (and most synchronizing functions such as Target Data) rely on RPC's exchanged between the Client and Server.
- Therefore, when executing an RPC (e.g. Gameplay Ability Activation) and performing operations on a replicated variable, you would never be guaranteed to have an up-to-date or stale value.

For more information, see the [EDC article on object replication order](https://dev.epicgames.com/documentation/en-us/unreal-engine/replicated-object-execution-order-in-unreal-engine).

If you believe you need a replicated variable, the solution is to instead use a Reliable RPC to send that data over.  Using a Reliable RPC will ensure proper ordering with the underlying synchronization mechanisms of GAS.

### Remote Procedure Calls in Gameplay Abilities

Remote Procedure Calls (RPC's) are the preferred method of communicating data between the client/server.  By making a Reliable RPC, you can ensure proper ordering with the other Gameplay Ability functions that support replication such as Activation.  There is currently no restriction against Unreliable RPC's, but know that order or delivery is not guaranteed.

Using a Multicast RPC will produce a validation warning (typically visible when compiling the Blueprint).  Since Gameplay Abilities never exist on Simulated Proxies, Multicast RPC's make little sense in the context of a Gameplay Ability.

### RPC Batching to Ensure Proper Activation and ReplicatedTargetData Order

There is a trick Epic uses internally to bundle the Gameplay Ability activation and Replicated Target Data.  Normally, if one were to implement a Locally Predicted Gameplay Ability that calls Activate() which in turn sets Replicated Target Data, the two would arrive at the Server in separate RPC's:

1. ServerTryActivateAbility (which will in turn call Activate)
2. ServerSetReplicatedTargetData (which will then set the data to the desired value -- but Activate has already run!)

There is a structure called [FScopedServerAbilityRPCBatcher](./Source/GameplayAbilities/Public/GameplayAbilityTypes.h) which is designed to use a single RPC to send both Activation and Target Data.  To use it, do the following:

1. In your `UAbilitySystemComponent`-derived class, override `ShouldDoServerAbilityRPCBatch` to return true.
2. In native code, Create an FScopedServerAbilityRPCBatcher on the stack.
3. Activate your Ability through your desired function (e.g. TryActivateAbility).
4. During the initial Activation of your ability, perform any CallServerSetReplicatedTargetData call.
5. When the destructor of the FScopedServerAbilityRPCBatcher executes (by going out of scope), it will call a batched RPC that contains both the Activation and the ReplicatedTargetData.

By using this structure, you will be guaranteed that the Server has the desired RPC data prior to calling the Gameplay Ability's activation function.

---

# Gameplay Effects {#gameplay-effects}

The purpose of Gameplay Effects is to modify an Actor in a predictable (and undoable) way.  Think of the verb Affect when you think of Gameplay Effects.  These are not Visual Effects or Sound Effects (those are called Gameplay Cues).  The Gameplay Effects are *applied* using [Gameplay Effect Specs](#gameplay-effect-specs) through the Ability System Component.

- Gameplay Effects that have a Duration (non-Instant) will automatically undo any modifications to the Actor upon removal. Instant ones will modify the Attribute's *BaseValue*.
- These are typically data-only Blueprints, though native implementations are also supported.
- They should be static after compile time; there is no way to modify them during runtime (Gameplay Effect Specs are the runtime version).
- They are essentially a complex datatable of things that should occur to a Target Actor when 'applied'.
- Composed from these pieces:
	- Duration / Timing data (such as how long the Effect lasts for, or how it periodically executes).
	- Rules for Stacking the Gameplay Effects.
	- Attribute Modifiers (data that controls how a Gameplay Attribute is modified and thus can be undone).
	- Custom Executions (a user definable function that executes every time a Gameplay Effect is applied).
	- Gameplay Effect Components (fragments of code / behavior to execute when applied).
	- Rules for applying Gameplay Cues (the VisualFX and AudioFX).
		
## Gameplay Effect Components {#gameplay-effect-components}

Gameplay Effect Components are introduced in UE5.3 to declutter the Gameplay Effect user interface and allow users of the Engine to provide their own game-specific functionality to Gameplay Effects.  

Read the interface for [UGameplayEffectComponent](./Source/GameplayAbilities/Public/GameplayEffectComponent.h)

## Gameplay Effect Specs {#gameplay-effect-specs}

These are the runtime wrapper structs for a Gameplay Effect.  They define the Gameplay Effect, any dynamic parameters (such as SetByCaller data), and the tags as they existed when the Spec was created.  The majority of the runtime API's use a *GameplayEffectSpec* rather than a *GameplayEffect*.

## Gameplay Effect Executions {#gameplay-effect-executions}

Gameplay Effect Executions are game-specific, user-written functions that are configured to execute when particular Gameplay Effects execute.  They typically read from and write to [Gameplay Attributes](#gameplay-attributes).  These are used when the calculations are much more complex than can be achieved with a simple attribute modifier.  Examples of this would be a damage system (see the [Lyra Example](https://dev.epicgames.com/documentation/unreal-engine/lyra-sample-game-in-unreal-engine)).

---

# Gameplay Tags {#gameplay-tags}

The Gameplay Ability System uses Gameplay Tags extensively throughout.  See the [official Developer Community documentation](https://dev.epicgames.com/documentation/unreal-engine/using-gameplay-tags-in-unreal-engine) for more details.

---

# Gameplay Cues {#gameplay-cues}

Gameplay Cues are a system for decoupling visual and audio fx from gameplay code.  On start-up, special Gameplay Cue asset folders are scanned for [Gameplay Cue Sets](./Source/GameplayAbilities/Public/GameplayCueSet.h), and *Gameplay Cue Notify* classes.

The implementer of a gameplay feature will either call the [Ability System Component](#asc)'s GameplayCue functions, or the [GameplayCueManager](./Source/GameplayAbilities/Public/GameplayCueManager.h)'s Gameplay Cue functions with a specific [Gameplay Tag](#gameplay-tag).  The effects artist will then create a *Gameplay Cue Notify* that corresponds to that tag.  The Gameplay Cue Manager is responsible for routing that specific tag to the proper *Gameplay Cue Notify*.

## Gameplay Cue replication

The details of Gameplay Cue replication are complex and worth noting.  Because these are cosmetic-only, there are unreliable RPC's that are used to communicate the execution of short-lived "*Burst*" cues.  We also use variable replication to synchronize the existance of longer cues (typically referred to as *Looping* or *Actor Notfies*).  This two-tiered approach ensures that Gameplay Cues can be dropped as unimportant, but also ensures important cues can be visible to any clients that *become relevant* according to the network system.

Due to these Gameplay Cues needing to obey network relevancy (i.e. far away players should not replicate their Cues, but newly relevant ones should) and the fact that the PlayerState is *always relevant*, there is a *replication proxy* system.  The Player's Pawn (who has its [ASC](#asc) on the PlayerState) should implement the [IAbilitySystemReplicationProxyInterface](./Source/GameplayAbilities/Public/AbilitySystemReplicationProxyInterface.h).  When turning on the ASC's ReplicationProxyEnabled variable, all unreliable Gameplay Cue RPC's will go through the proxy interface (the Pawn, which properly represents relevancy).

An advanced form of replication proxies also exists for the property replication so it may follow the same relevancy rules.  See `FMinimalGameplayCueReplicationProxy` in the [GameplayCueInterface](./Source/GameplayAbilities/Public/GameplayCueInterface.h).

Due to the Burst Cues being replicated by RPC and the Looping Cues being replicated by replicated variables, one can run into an issue where the unreliable burst RPC gets dropped but the looping events (OnBecomeRelevant/OnCeaseRelevant) arrive.  Less obvious, the unreliable OnBurst RPC can arrive but the OnBecomeRelevant/OnCeaseRelevant can be dropped if the Cue is removed on the server quick enough to result in no state changes for network serialization.

See the section on [Gameplay Cue Events](#gc-events) below for guidelines on how to implement your Gameplay Cue while taking into consideration network replication.

## Gameplay Cue Events {#gc-events}

When implementing a Gameplay Cue Notify Actor, the (legacy) naming of the functions may be confusing.  In UE5.5 the Blueprint (user-facing) names have changed in order to better represent what each function does.  They are laid out below.

### OnExecute

The execute function is the easiest to reason about:  It happens when you *Execute* a one-shot Gameplay Cue (aka a Static Notify / non-Looping Gameplay Cue).  The code path to Execute a Gameplay Cue (for Static Notifies) is different than the code path to Add a Gameplay Cue (for Looping Gameplay Cues aka Actor Notifies).

Due to the code path for execution being different, the caller of the Gameplay Cue must know that the receiver of the Gameplay Cue is a Static Notify in order for this to execute properly.  The call should route through ExecuteGameplayCue see [GameplayCueFunctionLibrary](./Source/GameplayAbilities/Public/GameplayCueFunctionLibrary.h).

### OnBurst (native: OnActive)

This event executes only once when a *Looping Gameplay Cue* first fires.  Due to it being delivered by unreliable RPC, it can be dropped silently by a client.  You can use this to implement cosmetic effects that are only relevant if a client witnessed the Gameplay Cue triggering.

### OnBecomeRelevant (native: WhileActive)

This event executes when the *Looping Gameplay Cue* first comes into network relevancy (usually when it's first added).  For instance, PawnA can have a Gameplay Cue activated, PawnB can join the game and still receive PawnA's OnBecomeRelevant -- but not receive OnBurst.

This is important to understand as OnBecomeRelevant and OnCeaseRelevant are both guaranteed to fire on the same Cue, whereas OnBurst is not guaranteed.

### OnCeaseRelevant (native: OnRemove)

This event executes when the *Looping Gameplay Cue* gets removed from network relevancy.  Usually that's when the server executes the removal of the Cue, but could also be when the client loses relevancy (e.g. by distance) of the viewed Cue.

In UE5.5, a warning is introduced if a Gameplay Cue implements OnBurst and OnCeaseRelevant and not OnBecomeRelevant.  The reasoning is that the opposite of OnCeaseRelevant is OnBecomeRelevant, not OnBurst and it's likely that the old naming scheme (OnActive/OnRemove) was a source of confusion.

---

# How Gameplay Prediction Works

There is documentation for how the Gameplay Prediction mechanisms work at the top of [GameplayPrediction.h](./Source/GameplayAbilities/Public/GameplayPrediction.h).

---

# Ability System Globals

There is a class called [AbilitySystemGlobals](./Source/GameplayAbilities/Public/AbilitySystemGlobals.h) which provide project customization points for how to handle specific base Ability System scenarios.  For example, there a functions you can override to implement derived classes of types used throughout the code (such as `AllocGameplayEffectContext`).

In UE5.5, a lot of these settings have started migrating to the [GameplayAbilitiesDeveloperSettings](./Source/GameplayAbilities/Public/GameplayAbilitiesDeveloperSettings.h) (which can be accessed using the Editor and choose the Project Settings menu item).  The rough division of responsibilities:  If it's a global setting (like a variable) then it should be configurable through Gameplay Abilities Developer Settings; if it's functionality (such as allocating project-specific classes) it should be in [AbilitySystemGlobals](./Source/GameplayAbilities/Public/AbilitySystemGlobals.h).

---

# Debugging the Gameplay Ability System

## Legacy ShowDebug Functionality

Prior to UE5.4, the way to debug the Gameplay Ability System was to use the "ShowDebug AbilitySystem" command.  Once there, you can cycle through the categories using the command `AbilitySystem.Debug.NextCategory` or explicitly choose a category using `AbilitySystem.Debug.SetCategory`.  This system is no longer maintained and may be deprecated in future versions.  You should instead be looking at the [Gameplay Debugger](#gameplay-debugger) functionality.

## Gameplay Debugger {#gameplay-debugger}

New in UE5.4, there is enhanced Gameplay Debugger functionality for the Gameplay Ability System.  This functionality is preferred over the ShowDebug system and should be your first line of defense in debugging GAS.  To enable it, open the Gameplay Debugger typically by using `shift-apostrophe` (`shift-'`) to select the locally controlled player, or simply the `apostrophe` (`'`) key to select the Actor that is closest to your reticule.

The debugger will show you the AbilitySystemComponent's current state as it pertains to Gameplay Tags, Gameplay Abilities, Gameplay Effects, and Gameplay Attributes.  In a networked game, the color coding helps to differentiate between how the server and client view their state.

## Console Commands

There are console commands that help both in developing and debugging GAS.  They are a great way to verify that your assumptions are correct about how abilities and effects should be activated, and coupled with the [Gameplay Debugger](#gameplay-debugger), what your state should be once executed.

All Ability System debug commands are prefixed with `AbilitySystem`.  The functionality we're reviewing here exists in the [AbilitySystemCheatManagerExtension](./Source/GameplayAbilities/Private/AbilitySystemCheatManagerExtension.cpp).  The source code also serves as an excellent reference  how to properly trigger the Gameplay Abilities and Gameplay Effects in native code (and what the expected results would be, depending on their configurations).

By implementing these in a Cheat Manager Extension, you are able to properly execute them as a local player, or on the server.  Many of the commands allow such a distinction with the `-Server` argument (read the command documentation or source code for more information).

One of the gotchas when using these commands is that the assets should be loaded prior to their use.  This is easily done in the Editor by simply right-clicking on the assets you want to use and clicking "Load Assets".

`AbilitySystem.Ability.Grant <ClassName/AssetName>` Grants an Ability to the Player.  Granting only happens on the Authority, so this command will be sent to the Server.
`AbilitySystem.Ability.Activate [-Server] <TagName/ClassName/AssetName>` Activate a Gameplay Ability.  Substring name matching works for Activation Tags (on already granted abilities), Asset Paths (on non-granted abilities), or Class Names on both.  Some Abilities can only be activated by the Client or the Server and you can control all of these activation options by specifying or ommitting the `-Server` argument.
`AbilitySystem.Ability.Cancel [-Server] <PartialName>` Cancels (prematurely Ends) a currently executing Gameplay Ability.  Cancelation can be initiated by either the Client or Server.
`AbilitySystem.Ability.ListGranted` List the Gameplay Abilities that are granted to the local player.  Since granting is done on the Server but replicated to the Client, these should always be in sync (so no option for -Server).

`AbilitySystem.Effect.ListActive [-Server]` Lists all of the Gameplay Effects currently active on the Player.
`AbilitySystem.Effect.Remove [-Server] <Handle/Name>` Remove a Gameplay Effect that is currently active on the Player.
`AbilitySystem.Effect.Apply [-Server] <Class/Assetname> [Level]` Apply a Gameplay Effect on the Player.  Substring name matching works for Asset Tags, Asset Paths, or Class Names.  Use -Server to send to the server (default is apply locally).

Gameplay Cues have their own set of debug commands.

`AbilitySystem.LogGameplayCueActorSpawning` Log when we create GameplayCueNotify_Actors.
`AbilitySystem.DisplayGameplayCues` Display GameplayCue events in world as text.
`AbilitySystem.GameplayCue.DisplayDuration` Configure the amount of time Gameplay Cues are drawn when `DisplayGameplayCues` is enabled.
`AbilitySystem.DisableGameplayCues` Disables all GameplayCue events in the world.
`AbilitySystem.GameplayCue.RunOnDedicatedServer` Run gameplay cue events on dedicated server.
`AbilitySystem.GameplayCueActorRecycle` Allow recycling of GameplayCue Actors.
`AbilitySystem.GameplayCueActorRecycleDebug` Prints logs for GC actor recycling debugging.
`AbilitySystem.GameplayCueCheckForTooManyRPCs` Warns if gameplay cues are being throttled by network code.

## Visual Logger

New in UE5.4, there has been extra care put into the [Visual Logger](https://dev.epicgames.com/documentation/en-us/unreal-engine/visual-logger-in-unreal-engine) facilities for the Gameplay Ability System.  The [Visual Logger](https://dev.epicgames.com/documentation/en-us/unreal-engine/visual-logger-in-unreal-engine) is useful to see the complex interactions of Gameplay Abilities and Gameplay Effects over time.  The Visual Logger always captures the verbose logs and saves a snapshot of the state of the Ability System Component on every frame there is a log entry.

In UE5.4, the [Visual Logger](https://dev.epicgames.com/documentation/en-us/unreal-engine/visual-logger-in-unreal-engine) now correctly orders the events between clients and servers when using Play In Editor.  This makes the Visual Logger especially useful for debugging how the client and server interact when activating abilities, gameplay effects, and modifying attributes.