// --------------------------------------------------------------------------------------------------------------------
//	Network Prediction Plugin Overview
// --------------------------------------------------------------------------------------------------------------------

NetworkPrediction is a generalized system for client-side prediction. The goal here is to separate the gameplay code
from the networking code: prediction, corrections, rollbacks, re-simulation, etc. Ideally, gameplay simulation code 
can be agnostic to networking and prediction. Branches like "IsServer()" or "IsResimulating()" should not be necessary.

At the core of the system are user states and a SimulationTick function. User states are divided into three buckets. 
These are implemented as structs:

	InputCmd: The state that is generated by a controlling client.
	SyncState: The state that primarily evolves frame-to-frame via a SimulationTick function.
	AuxState: Additional state that can change during SimulationTick, but infrequently.

Given these state types, user then implements a SimulationTick function which takes an input {Inputcmd, Sync, Aux} and
produces output {Sync, Aux}. These inputs and outputs are what is networked.

NetworkPredictionExtras is a supplementary plugin with sample content.


// -------------------------------------------------------------
//	Getting Started
// -------------------------------------------------------------

The NetworkPredictionExamples plugin contains a variety of working examples. You can access them by adding the
NetworkPrediction and NetworkPredictionExamples plugins to (almost) any project. 

TestMap_Empty is a good starting point, featuring a simple pawn using a "flying" movement simulation.

Some starting points for exploring under the hood:

	MockNetworkSimulation.h - entry point for simple example use case. See how a simple simulation is defined and how
	an actor component is bound to it at runtime.

	NetworkPredictionWorldManager.h - top level entry point for the system. See what happens each frame,
	how simulations are managed and coordinated.

	NetworkPredictionPhysicsComponent.h - Example of binding a physics-only sim.

	MockPhysicsSimulation.h - Simple "controllable physics object" example.

NetworkPredictionInsights is a supplementary plugin tool that can be used to trace and display detailed simulation
state information, including rollbacks.  Once added to your project, launching it with -trace=np will enable tracing
by default.  You can open the tool from Unreal Editor's Tool menu -> "Network Prediction Insights".


// -------------------------------------------------------------
//	High-Level Architecture and Operational Flow
// -------------------------------------------------------------

The Network Prediction plugin's framework is built to support multiple Simulation Instances at once.

A Simulation Instance is an abstract element within NPP's framework that is initialized and configured when user code
calls NetworkPredictionProxy::Init<>().  It consists of a Model Definition, along with a Simulation that evolves the 
instance over time and a Driver that interfaces with the game world representation of the instance. Typically a
Simulation Instance is paired exclusively with a single Actor in the game world.  

Refer to the example section below for a concrete use case.

The Model Definition:
	- InputCmd type: the unit of input authored by the owning client (or server)
	- SyncState type: the server-authoritative state that is expected to change almost every frame
	- AuxState type: the server-authoritative state that is expected to change infrequently

The Simulation: the class that defines the SimulationTick function for our Model Definition. This could even be the
same as the Driver.
	Runs SimulationTick: given a set of {InputCmd, SyncState, AuxState} and a timestep, produce a new
	{SyncState, AuxState}

The Driver: typically an ActorComponent, but could be an Actor or any object. It has the following responsibilities:
	Producing initial {SyncState, AuxState} when the simulation starts
	Producing InputCmds every sim frame (e.g. reading controller input to fill out input properties)
	Translating {SyncState, AuxState} to game state. (e.g. taking the SyncState's position & rotation, and applying it
	as a transform to an Actor)

The NetworkPredictionWorldManager orchestrates all operations on simulation instances, including initialization,
replication, buffering input & state, etc.  It does this using a collection of services that do work on groups of
simulation instances. Refer to NetworkPredictionServiceRegistry.h for more details.

All operations are performed on the game thread, driven by specific game world delegates like OnWorldPreActorTick and
PostTickDispatchEvent. Each game tick, the framework decides how many fixed ticks to execute depending on the engine
frame delta time, and then for each tick:

	1) For each remotely-controlled simulation instance, produce remote input (received and stored by the framework)
	2) For each locally-controlled simulation instance, produce local input (to use and send to networked peers)
	3) Tick each simulation instance in order

The order with which the instances will be ticked depends on the simulation sort priority (see 
ENetworkPredictionSortPriority). This is defined optionally as part of the instance’s ModelDef, and because all fixed
rate instances are stepped together, it allows us to define priority ordering of instances. This ordering is consistent
between all clients and server.

For instances locally operating on clients as ENetworkLOD::Interpolated, the framework also calls Interpolate() on the
Driver, in order to produce results aligning with the rest of the simulation framework’s timeline from authoritative 
state snapshots.

FinalizeFrame() is then called on the Driver. The goal of this function is to publish simulation state to the game 
presentation layers, for instance actually modifying the actor position based on state results.

When a state divergence is identified and a reconciliation is requested, NPP will go through 
UNetworkPredictionWorldManager::ReconcileSimulationsPostNetworkUpdate(). If requested, this will happen at the very 
beginning of the client frame, before advancing the instances through 
UNetworkPredictionWorldManager::BeginNewSimulationFrame(). 

Reconciliation will occur differently depending on the simulation instance:

	ENetworkLOD::Interpolated instances will directly reconcile to the new state using Interpolate().
	ENetworkLOD::ForwardPredict instances will collectively do a rollback. A RollbackFrame is identified depending on
		whichever was the earliest frame among all instances that requested a rollback, and then ALL fixed rate
		simulation instances are rolled back to that frame, and stepped forward in the correct 
		ENetworkPredictionSortPriority until we are back to the predicted timeline. 
		
Note that during rollback, all fixed rate instances are rolled back and resimulated as a global operation. If multiple 
instances are running on a client, all of them will be rolled back, allowing developers to maintain dependencies between 
instance states (i.e. one simulation instance could have an effect that influences another instance’s input).

See also UNetworkPredictionWorldManager::ConfigureInstance for how a simulation instance is configured.
See also UNetworkPredictionWorldManager::BeginNewSimulationFrame for understanding how a simulation is advanced.


// -------------------------------------------------------------
//	Example: Top-down Character Locomotion Driven By Player Input
// -------------------------------------------------------------

To understand NPP’s basic architecture better, it may be useful to break down how an example simulation could be set 
up. Take a hypothetical setup of a top-down character game where the player can walk around the world in any direction,
and hold down a button to sprint faster.

The Model Definition:
	- InputCmd: contains a directional intent vector, and a boolean indicating whether attempting to sprint or not
	- SyncState: contains position, facing direction, velocity
	- AuxState: contains max speed

The Driver type for this instance would be an Actor component, inheriting from UNetworkPredictionComponent. It features
a few interesting functions:
	- InitializeSimulationState: this is called before simulation begins and produces the first {SyncState, AuxState} 
		based on the owning Actor's initial spawn transform and some data-driven game settings.
	- ProduceInput: this reads controller input to determine directional intent and sprinting button state. It is 
		called only on the owning client.
	- FinalizeFrame/RestoreFrame: this takes {SyncState, AuxState} and applies the position and facing direction to the
		actor. The velocity is cached as a property of the component.
	- GetVelocity: provides read-only access to the most-recent velocity.

The Simulation type would be a class that stores no state and has only a SimulationTick function:
	- 'In' Parameters: {InputCmd, SyncState, AuxState}, TimeStep
	- 'Out' Parameters: {NewSyncState, NewAuxState}
	- Based on whether attempting sprint, adjust NewAuxState's MaxSpeed based on data-driven game settings
	- Read directional intent vector, and translate that to a desired velocity using MaxSpeed
	- Compute NewSyncState's velocity based on some acceleration method
	- Compute a move delta based on velocity and the TimeStep, then attempt to move that amount and face that direction
	- Capture the new position and facing direction in NewSyncState

The character pawn also has a mesh with an animation graph. It uses the Driver component's GetVelocity query to make
sure its animation rate reflects how fast the character is moving.

To allow proper synchronization across peers, all instances of this character would operate using 
ENetworkPredictionTickingPolicy::Fixed.

On the controlling player's client, where the character's role is an Autonomous Proxy, this simulation would be
operating in ENetworkLOD::ForwardPredict mode, so that local InputCmds can immediately be applied and movement can be
forward-predicted for responsiveness. The owning client sends these InputCmds to the server.

On the server, received InputCmds are added to a buffer (typically only 2-3 frames, but it will expand and contract as
conditions change). As the server ticks, it peels off the next buffered InputCmd and uses it for its own simulation.
The resulting {SyncState, AuxState} is sent to all clients.

On the other clients, where the character's role is a Simulated Proxy, this simulation is operating in
ENetworkLOD::Interpolated mode. Incoming state {SyncState, AuxState} from the server is buffered, and current state is
an interpolation between the 2 most relevant frames, determined by the current simulation time.

Back on the controlling player's client, we receive an authoritative {SyncState, AuxState} from the server for a 
particular frame and compare it against what was locally-predicted. If they match within reason, consider the frame
acknowledged. If they don't match, trigger a rollback and re-simulate using our locally-stored InputCmds. 


// -------------------------------------------------------------
//	More Details
// -------------------------------------------------------------

Fixed Ticking: all clients and server agree on a ticking rate. Real time is accumulated and then simulations tick in
	fixed steps. 
	
	Advantage: clients can accurately predict any simulation since everything ticks together. 
	
	Disadvantage: requires server-side buffering of InputCmds, which increases client-server lag. Requires local
		frame smoothing for smooth on-screen motion, which is not implemented yet.
	
Independent Ticking: all clients tick at their own local variable framerate and send InputCmds to the server at that
	rate, with time deltas included. The server ticks the client-owned objects as InputCmds arrive, using their time
	deltas. This is similar to how UE's Character Movement Component works.
	
	Advantage: low latency, server can process InputCmds as soon as they're received without buffering. Reduces the
	   number of predicted frames kept on the client. No need for local frame smoothing.
	   
	Disadvantage: clients can't accurately predict objects they don't own, due to ticking rate differences.

Prediction Terminology: 'predict' is a term loosely thrown around. Network Prediction uses these terms:

	Forward Prediction: clients predicting the state of game object ahead of where the server is. More precisely, this
		is a client predicting the state that game objects will be in when the server processes the client's InputCmd
		that the client is processing locally.
		
	Interpolation: always meaning to blend between states received from the server and NOT running simulation code to
		generate new state data. 
		
	Simulation Extrapolation (not supported yet!): taking network updates from the server and extrapolating subsequent
		frames by running simulation code, rather than simply blending states.
    
Prediction and Corrections: only the client performs these operations. Clients may forward predict the state of objects
	ahead of where the server is, and perform corrections if they receive server-authoritative state that disagrees
	with their predicted state. But the server simply generates its own state, using InputCmds received from autonomous
	clients. This is a key difference from UE's Character Movement Component, where the server detects and issues 
	corrections to character moves.

Networked Simulation Cues: This is a system for replicating events that do not affect the simulation state, such as
	visual or audio events. These cues provide hooks for developers to implement their own rewindable, undoable, 
	time-aware events that can be tailored how they are replicated and predicted.
	Refer to NetworkPredictionCues.h for more details.
	
Network Serialization in NPP: this works through normal Unreal property replication using a custom Net Serializer, and
	various replication operations are redirected using FReplicationProxy objects to serialize and de-serialize from an
	FArchive that contains the actual bits that get transmitted. Some RPC functions are used as well, such as when 
	autonomous clients send input to the server.


// -------------------------------------------------------------
//	Caveats, Known Issues, Missing Features
// -------------------------------------------------------------

Fixed Tick Simulations and Variable Rendering Rate
	There is currently no support for interpolating locally-owned instances for smoother movement presentation. 
	e.g. if you wanted to run simulations at a fixed 30 hz on server and all clients but render at a variable 60-200 hz
	A new smoothing / correction service would be useful here, fulfilling these roles:
		- Handle general smoothing when in fixed tick mode with a different rendering rate
		- Handle smoothing towards a corrected state. Currently all corrections are instant and jarring.

Reflection-Based Data Modeling
	It takes a signficant amount of boilerplate code to define the model definition types and their functionality.
	This could be reduced by implementing some kind of property markup scheme that provided default interpolation
	methods with the ability to override them.

General Performance
	At present, expect performance characteristics to be worse than default Unreal replication in terms of client-side
	CPU (for re-simulation) and bandwidth (for safety/correctness in redundant sending of unacknowledged data).
	Although the architecture is aimed at being cache-friendly, little time has been spent on profiling and
	optimization.

Throttling Fixed Simulation Steps
	There is currently no throttling of fixed tick simulation steps during slow framerate situations, such as a hitch
	during asset loading. This could cause a large amount of simulation to occur during a single render frame, leading
	to additional performance degradation. In the worst cases, this can spiral and grind the game to a halt. Providing
	options to limit the amount of simulation time consumed on a render frame would help. In some cases where a client
	experiences a large hitch, a full reconciliation may be appropriate to re-establish synchronization between client
	and server.

Server-side Input Buffering
	Although input buffering correctly increases as network conditions degrade, the current implementation does not
	offer customization of methods of shrinking the buffer following recovery from bad networking conditions, whether
	through dropping non-critical inputs, merging them, or some other game-specific approach. 
	It could benefit from additional options, such as control over 'target' server-side buffer size and client-side
	input sending redundancy.

AuxState Handling
	The intent of breaking AuxState out into a separate object is to allow more efficient sparse storage. But the
	current implementation does not store it sparsely or replicate it differently than SyncState. A goal is to 
	implement this in the future, which should come as an under-the-hood change with no modifications required at the
	game project level.

Server-Side Rewind Lag Compensation
	This is a common feature in competitive networked action games, accounting for varying latency of clients. It's
	necessary in Independent ticking mode to do things like server-authoritative hitscan weapons. The server
	temporarily rewinds sim instances that the client was interpolating to their past state, so that computations can
	be performed as the client saw them. It's important to note that the server would never alter any of its past
	frames.

Async Network Prediction Has Been Removed
	After a long road and many attempts, we are dropping support for the async version of Network Prediction. We felt
	the complications it introduced into the physics system were too much to maintain and performance was still too
	poor in the worst/degenerate cases that it wasn't going to be a viable system for enough games to warrant the 
	complexities.
	
	The original single threaded version of Network Prediction is preserved and unchanged. We still hope to use it to
	build a new character moverment system with it. Physics support could come back into this version but it would be
	strictly opt-in and only applicable to games with small number of objects and players.