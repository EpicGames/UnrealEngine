@ECHO OFF

REM *** check environment ***
IF "%GameDK%"=="" GOTO no_gdk

SET WDAPP="%GameDK%bin\wdapp.exe"
IF NOT EXIST %WDAPP% GOTO no_wdapp


REM *** remove existing ***
call "%~dp0__UNINSTALLBATCHFILENAME__" > nul


REM *** install new ***
ECHO Installing __PROJECTNAME__ from %~dp0__PACKAGEFILENAME__
%WDAPP% install "%~dp0__PACKAGEFILENAME__"

GOTO :eof



:no_gdk
Echo could not find GameDK environment variable - is the GDK installed?
GOTO :eof


:no_wdapp
Echo could not find %WDAPP% - is the GDK installed?
GOTO :eof


