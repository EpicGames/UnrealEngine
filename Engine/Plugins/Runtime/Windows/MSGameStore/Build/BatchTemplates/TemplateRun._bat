@ECHO OFF
IF "%GameDK%"=="" GOTO no_gdk

SET WDAPP="%GameDK%bin\wdapp.exe"
IF NOT EXIST %WDAPP% GOTO no_wdapp


REM *** find the AUMID matching the the identity name and publisher ***
%WDAPP% list  |   find /I "__PACKAGEIDENTITYNAME_____PACKAGEIDENTITYPUBLISHER__!" > pkg.tmp
SET AUMID=
SET /P AUMID=<pkg.tmp
DEL pkg.tmp
IF "%AUMID%"=="" goto no_package


REM *** escape any quotes in the command line arguments ***
SET ARG=%*
IF DEFINED ARG ( SET ARG=%ARG:"=\"% )


REM *** launch it ***
ECHO.
ECHO Running __PROJECTNAME__ via %AUMID%
%WDAPP% launch %AUMID% %ARG%
GOTO :eof


:no_package
ECHO __PROJECTNAME__ does not appear to be installed
GOTO :eof


:no_gdk
ECHO could not find GameDK environment variable - is the GDK installed?
GOTO :eof


:no_wdapp
ECHO could not find %WDAPP% - is the GDK installed?
GOTO :eof


