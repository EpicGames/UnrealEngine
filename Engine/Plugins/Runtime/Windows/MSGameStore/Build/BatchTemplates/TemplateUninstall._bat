@ECHO OFF

REM *** check environment ***
IF "%GameDK%"=="" GOTO no_gdk

SET WDAPP="%GameDK%bin\wdapp.exe"
IF NOT EXIST %WDAPP% GOTO no_wdapp



REM *** find the package full name matching the identity name and publisher ***
%WDAPP% list  |   find /I "__PACKAGEIDENTITYNAME__"   |   find /I "__PACKAGEIDENTITYPUBLISHER__"   |   find /V "!"   >   pkg.tmp
SET PACKAGE=
SET /P PACKAGE=<pkg.tmp
DEL pkg.tmp
IF "%PACKAGE%"=="" GOTO no_package



REM *** uninstall the package ***
ECHO uninstalling %PACKAGE% for __PROJECTNAME__...
%WDAPP% uninstall %PACKAGE%

GOTO :eof


:no_package
ECHO ...__PROJECTNAME__ does not appear to be installed
GOTO :eof


:no_gdk
ECHO could not find GameDK environment variable - is the GDK installed?
GOTO :eof


:no_wdapp
ECHO could not find %WDAPP% - is the GDK installed?
GOTO :eof


