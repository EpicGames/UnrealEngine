<?xml version="1.0" encoding="utf-8"?> 
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">

  <Type Name="FZoneGraphLaneLocation">
    <DisplayString>(seg[{LaneSegment}] hdl[{LaneHandle.Index}:{LaneHandle.DataHandle.Index}] dist[{DistanceAlongLane}])</DisplayString>
  </Type>

  <Type Name="FZoneGraphCrowdLaneStateChangeEvent">
    <DisplayString>Lane={Lane.Index} NewState={State}</DisplayString>
  </Type>

</AutoVisualizer>